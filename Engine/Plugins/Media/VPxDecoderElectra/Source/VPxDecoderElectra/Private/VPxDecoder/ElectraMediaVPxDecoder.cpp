// Copyright Epic Games, Inc. All Rights Reserved.

#include "ElectraMediaVPxDecoder.h"
#include "VPxDecoderElectraModule.h"
#include "IElectraCodecRegistry.h"
#include "IElectraCodecFactory.h"
#include "Misc/ScopeLock.h"
#include "Templates/SharedPointer.h"
#include "Features/IModularFeatures.h"
#include "Features/IModularFeature.h"
#include "IElectraCodecFactoryModule.h"
#include "IElectraDecoderFeaturesAndOptions.h"
#include "IElectraDecoder.h"
#include "IElectraDecoderOutputVideo.h"
#include "ElectraDecodersUtils.h"
#include "Utils/Google/ElectraUtilsVPxVideo.h"
#include "Utils/Google/ElectraBitstreamProcessorVPx.h"
#include "Containers/Queue.h"
#include <Stats/Stats.h>


#define ENABLE_GPU_BUFFER_HELPER 1

#if ENABLE_GPU_BUFFER_HELPER
#include "ElectraDecodersPlatformResources.h"
#include COMPILED_PLATFORM_HEADER(ElectraTextureSampleGPUBufferHelper.h)
#else
#define ELECTRA_MEDIAGPUBUFFER_DX12 0
#endif

#include <vpx_decoder.h>
#include <vp8dx.h>

/*********************************************************************************************************************/


/*********************************************************************************************************************/

DECLARE_CYCLE_STAT(TEXT("ElectraDecoder ConvertOutput"), STAT_ElectraDecoder_ConvertOutputVpx, STATGROUP_Media);

/*********************************************************************************************************************/

#define ERRCODE_INTERNAL_NO_ERROR							0
#define ERRCODE_INTERNAL_ALREADY_CLOSED						1
#define ERRCODE_INTERNAL_COULD_NOT_CREATE_DECODER			2
#define ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT				3
#define ERRCODE_INTERNAL_FAILED_TO_FLUSH_DECODER			4
#define ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE	5

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

namespace DynamicSidebandData
{
	const FName VPxAlpha(TEXT("vpx-alpha"));
}


class FVideoDecoderVPxElectra;


class FVideoDecoderOutputVPxElectra : public IElectraDecoderVideoOutput, public IElectraDecoderVideoOutputImageBuffers
{
public:
	virtual ~FVideoDecoderOutputVPxElectra()
	{ }

	FTimespan GetPTS() const override
	{ return PTS; }
	uint64 GetUserValue() const override
	{ return UserValue; }

	EOutputType GetOutputType() const
	{ return EOutputType::Output; }
	int32 GetWidth() const override
	{ return Width - Crop.Left - Crop.Right; }
	int32 GetHeight() const override
	{ return Height - Crop.Top - Crop.Bottom; }
	int32 GetDecodedWidth() const override
	{ return DecodedWidth; }
	int32 GetDecodedHeight() const override
	{ return DecodedHeight; }
	FElectraVideoDecoderOutputCropValues GetCropValues() const override
	{ return Crop; }
	int32 GetAspectRatioW() const override
	{ return AspectW; }
	int32 GetAspectRatioH() const override
	{ return AspectH; }
	int32 GetFrameRateNumerator() const override
	{ return FrameRateN; }
	int32 GetFrameRateDenominator() const override
	{ return FrameRateD; }
	int32 GetNumberOfBits() const override
	{ return NumBits; }
	void GetExtraValues(TMap<FString, FVariant>& OutExtraValues) const override
	{ OutExtraValues = ExtraValues; }
	void* GetPlatformOutputHandle(EElectraDecoderPlatformOutputHandleType InTypeOfHandle) const override
	{
		if (InTypeOfHandle == EElectraDecoderPlatformOutputHandleType::ImageBuffers)
		{
			return static_cast<IElectraDecoderVideoOutputImageBuffers*>(const_cast<FVideoDecoderOutputVPxElectra*>(this));
		}
		return nullptr;
	}
	IElectraDecoderVideoOutput::EImageCopyResult CopyPlatformImage(IElectraDecoderVideoOutputCopyResources* InCopyResources) const override
	{ return IElectraDecoderVideoOutput::EImageCopyResult::NotSupported; }

	// Methods from IElectraDecoderVideoOutputImageBuffers
	uint32 GetCodec4CC() const override
	{ return Codec4CC; }
	int32 GetNumberOfBuffers() const override
	{
		return NumBuffers;
	}
	TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> GetBufferDataByIndex(int32 InBufferIndex) const override
	{
		if (InBufferIndex == 0)
		{
			return ColorBuffer;
		}
		return nullptr;
	}
	void* GetBufferTextureByIndex(int32 InBufferIndex) const override
	{
#if ELECTRA_MEDIAGPUBUFFER_DX12
		if (InBufferIndex == 0)
		{
			return GPUBuffer.Resource.GetReference();
		}
#endif
		return nullptr;
	}
	virtual bool GetBufferTextureSyncByIndex(int32 InBufferIndex, FElectraDecoderOutputSync& SyncObject) const override
	{
#if ELECTRA_MEDIAGPUBUFFER_DX12
		if (InBufferIndex == 0)
		{
			SyncObject = { GPUBuffer.Fence.GetReference(), GPUBuffer.FenceValue };
			return true;
		}
#endif
		return false;
	}
	EPixelFormat GetBufferFormatByIndex(int32 InBufferIndex) const override
	{
		if (InBufferIndex == 0)
		{
			return ColorBufferFormat;
		}
		return EPixelFormat::PF_Unknown;
	}
	EElectraTextureSamplePixelEncoding GetBufferEncodingByIndex(int32 InBufferIndex) const override
	{
		if (InBufferIndex == 0)
		{
			return ColorBufferEncoding;
		}
		return EElectraTextureSamplePixelEncoding::Native;
	}
	int32 GetBufferPitchByIndex(int32 InBufferIndex) const override
	{
		if (InBufferIndex == 0)
		{
			return ColorPitch;
		}
		return 0;
	}

public:
	FTimespan PTS;
	uint64 UserValue = 0;

	FElectraVideoDecoderOutputCropValues Crop;
	int32 Width = 0;
	int32 Height = 0;
	int32 DecodedWidth = 0;
	int32 DecodedHeight = 0;
	int32 NumBits = 0;
	int32 AspectW = 1;
	int32 AspectH = 1;
	int32 FrameRateN = 0;
	int32 FrameRateD = 0;
	TMap<FString, FVariant> ExtraValues;

	uint32 Codec4CC = 0;
	int32 NumBuffers = 0;
	TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> ColorBuffer;
	EPixelFormat ColorBufferFormat;
	EElectraTextureSamplePixelEncoding ColorBufferEncoding;
	int32 ColorPitch = 0;
#if ELECTRA_MEDIAGPUBUFFER_DX12
	FElectraMediaDecoderOutputBufferPool_DX12::FOutputData GPUBuffer;
#endif
};


class FVideoDecoderVPxElectra : public IElectraDecoder
{
public:
	static void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions)
	{
		OutOptions.Emplace(IElectraDecoderFeature::MinimumNumberOfOutputFrames, FVariant((int32)5));
		OutOptions.Emplace(IElectraDecoderFeature::IsAdaptive, FVariant(false));
	}

	FVideoDecoderVPxElectra(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions);
	virtual ~FVideoDecoderVPxElectra();

	IElectraDecoder::EType GetType() const override
	{ return IElectraDecoder::EType::Video; }

	void GetFeatures(TMap<FString, FVariant>& OutFeatures) const override;

	FError GetError() const override;

	void Close() override;
	IElectraDecoder::ECSDCompatibility IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions) override;
	bool ResetToCleanStart() override;

	EDecoderError DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions) override;
	EDecoderError SendEndOfData() override;
	EDecoderError Flush() override;
	EOutputStatus HaveOutput() override;
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> GetOutput() override;

	TSharedPtr<IElectraDecoderBitstreamProcessor, ESPMode::ThreadSafe> CreateBitstreamProcessor() override
	{
		TMap<FString, FVariant> DecoderFeatures;
		GetFeatures(DecoderFeatures);
		return FElectraDecoderBitstreamProcessorVPx::Create(DecoderFeatures, InitialCodecFormat, InitialCreationOptions);
	}

	void Suspend() override
	{ }
	void Resume() override
	{ }

private:
	struct FDecoderInput
	{
		~FDecoderInput()
		{
			if (InputDataCopy)
			{
				FMemory::Free(InputDataCopy);
			}
			if (InputAlphaDataCopy)
			{
				FMemory::Free(InputAlphaDataCopy);
			}
		}
		FInputAccessUnit AccessUnit;
		TMap<FString, FVariant> AdditionalOptions;
		void* InputDataCopy = nullptr;
		int32 InputDataSize = 0;
		void* InputAlphaDataCopy = nullptr;
		int32 InputAlphaDataSize = 0;
		int32 SuperFrameIndexPlusOne = 0;
	};

	enum class EDecodeState
	{
		Decoding,
		Draining
	};

	static constexpr uint32 Make4CC(const uint8 A, const uint8 B, const uint8 C, const uint8 D)
	{
		return (static_cast<uint32>(A) << 24) | (static_cast<uint32>(B) << 16) | (static_cast<uint32>(C) << 8) | static_cast<uint32>(D);
	}

	bool PostError(int32 ApiReturnValue, FString Message, int32 Code);

	bool InternalDecoderCreate(const TMap<FString, FVariant>& InAdditionalOptions);
	void InternalDecoderDestroy();

	bool DecomposeSuperFrame(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions);
	bool PrepareSingleFrame(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions);
	IElectraDecoder::EDecoderError DecodeNextPending();

	enum class EConvertResult
	{
		Success,
		Failure,
		GotEOS
	};
	EConvertResult ConvertDecoderOutput(vpx_image_t* InDecodedImage, vpx_image_t* InDecodedAlphaImage);

	bool ConvertDecodedImageToNV12orP010(FVideoDecoderOutputVPxElectra* InNewOutput, const vpx_image_t* InDecodedImage, void* PlatformDevice, int32 PlatformDeviceVersion) const;
	bool ConvertDecodedImageWithAlpha(FVideoDecoderOutputVPxElectra* InNewOutput, const vpx_image_t* InDecodedImage, const vpx_image_t* InDecodedAlphaImage, void* PlatformDevice, int32 PlatformDeviceVersion) const;

	Electra::FCodecTypeFormat InitialCodecFormat;
	TMap<FString, FVariant> InitialCreationOptions;

	TQueue<TSharedPtr<FDecoderInput, ESPMode::NotThreadSafe>, EQueueMode::Spsc> PendingDecoderInput;
	TArray<TSharedPtr<FDecoderInput, ESPMode::NotThreadSafe>> InDecoderInput;
	TSharedPtr<FVideoDecoderOutputVPxElectra, ESPMode::ThreadSafe> CurrentOutput;
	IElectraDecoder::FError LastError;
	EDecodeState DecodeState = EDecodeState::Decoding;

	enum EDecoderType
	{
		Color = 0,
		Alpha = 1
	};

	struct FDecoderStructs
	{
		vpx_codec_ctx_t Context;
		vpx_codec_ctx_t* Handle = nullptr;
		vpx_codec_iter_t OutputIterator = nullptr;
	};
	FDecoderStructs Decoders[2];

	uint32 MaxWidth;
	uint32 MaxHeight;
	uint32 MaxOutputBuffers;
#if ELECTRA_MEDIAGPUBUFFER_DX12
	mutable TSharedPtr<FElectraMediaDecoderOutputBufferPool_DX12, ESPMode::ThreadSafe> D3D12ResourcePool;
#endif
};

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

class FVPxVideoDecoderElectraFactory : public IElectraCodecFactory, public IElectraCodecFactory::IProviderInformation, public IElectraCodecModularFeature, public TSharedFromThis<FVPxVideoDecoderElectraFactory, ESPMode::ThreadSafe>
{
public:
	virtual ~FVPxVideoDecoderElectraFactory()
	{ }

	FVPxVideoDecoderElectraFactory()
	{
		LibVersion = ANSI_TO_TCHAR(vpx_codec_version_str());
	}

	void GetListOfFactories(TArray<TWeakPtr<IElectraCodecFactory, ESPMode::ThreadSafe>>& OutCodecFactories) override
	{
		OutCodecFactories.Add(AsShared());
	}

	void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions) const override
	{
		FVideoDecoderVPxElectra::GetConfigurationOptions(OutOptions);
	}

	int32 SupportsDecoding(TMap<FString, FVariant>& OutFormatInfo, const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) const override
	{
		if (InCodecFormat.Type != Electra::FCodecTypeFormat::EType::Video || !InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FVideo>())
		{
			return 0;
		}
		const Electra::FCodecTypeFormat::FVideo& vi = InCodecFormat.Properties.Get<Electra::FCodecTypeFormat::FVideo>();
		if (InCodecFormat.RFC6381.StartsWith(TEXT("vp08"), ESearchCase::IgnoreCase))
		{
			return 1;
		}
		else if (InCodecFormat.RFC6381.StartsWith(TEXT("vp09"), ESearchCase::IgnoreCase))
		{
			if (vi.BitDepth > 8)
			{
			#if defined(VPX_CODEC_CAP_HIGHBITDEPTH)
				return !!(vpx_codec_get_caps(vpx_codec_vp9_dx()) & VPX_CODEC_CAP_HIGHBITDEPTH);
			#else
				return 0;
			#endif
			}
			return 1;
		}
		return 0;
	}

	TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> CreateDecoder(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) override
	{
		return InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FVideo>() ? MakeShared<FVideoDecoderVPxElectra, ESPMode::ThreadSafe>(InCodecFormat, InAdditionalOptions) : TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe>();
	}


	const IProviderInformation& GetProviderInformation() const override
	{ return *this; }
	FString GetName() const override
	{ return FString(TEXT("libvpx")); }
	FString GetVersion() const override
	{ return LibVersion; }
	FString GetImplementation() const override
	{ return(FString()); }
	FString GetVendor() const override
	{ return(FString()); }

	FString LibVersion;
	static TSharedPtr<FVPxVideoDecoderElectraFactory, ESPMode::ThreadSafe> Self;
};
TSharedPtr<FVPxVideoDecoderElectraFactory, ESPMode::ThreadSafe> FVPxVideoDecoderElectraFactory::Self;

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

void FElectraMediaVPxDecoder::Startup()
{
	// Make sure the codec factory module has been loaded.
	FModuleManager::Get().LoadModule(TEXT("ElectraCodecFactory"));

	// Create an instance of the factory, which is also the modular feature.
	check(!FVPxVideoDecoderElectraFactory::Self.IsValid());
	FVPxVideoDecoderElectraFactory::Self = MakeShared<FVPxVideoDecoderElectraFactory, ESPMode::ThreadSafe>();
	// Register as modular feature.
	IModularFeatures::Get().RegisterModularFeature(IElectraCodecFactoryModule::GetModularFeatureName(), FVPxVideoDecoderElectraFactory::Self.Get());
}

void FElectraMediaVPxDecoder::Shutdown()
{
	IModularFeatures::Get().UnregisterModularFeature(IElectraCodecFactoryModule::GetModularFeatureName(), FVPxVideoDecoderElectraFactory::Self.Get());
	FVPxVideoDecoderElectraFactory::Self.Reset();
}

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

FVideoDecoderVPxElectra::FVideoDecoderVPxElectra(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions)
	: InitialCodecFormat(InCodecFormat)
	, InitialCreationOptions(InAdditionalOptions)
{
	const Electra::FCodecTypeFormat::FVideo& vid(InitialCodecFormat.Properties.Get<Electra::FCodecTypeFormat::FVideo>());
	MaxWidth = (uint32)Align(ElectraDecodersUtil::GetVariantValueSafeU64(InitialCreationOptions, TEXT("max_width"), 1920), 2);
	MaxHeight = (uint32)Align(ElectraDecodersUtil::GetVariantValueSafeU64(InitialCreationOptions, TEXT("max_height"), 1080), 2);
	MaxOutputBuffers = (uint32)ElectraDecodersUtil::GetVariantValueSafeU64(InitialCreationOptions, TEXT("max_output_buffers"), 5);
#if ELECTRA_MEDIAGPUBUFFER_DX12
	MaxOutputBuffers += kElectraDecoderPipelineExtraFrames;
#endif
}

FVideoDecoderVPxElectra::~FVideoDecoderVPxElectra()
{
	// Close() must have been called already!
	check(LastError.Code == ERRCODE_INTERNAL_ALREADY_CLOSED);
	// We do it nonetheless...
	Close();
}

void FVideoDecoderVPxElectra::GetFeatures(TMap<FString, FVariant>& OutFeatures) const
{
	GetConfigurationOptions(OutFeatures);
}

IElectraDecoder::FError FVideoDecoderVPxElectra::GetError() const
{
	return LastError;
}

bool FVideoDecoderVPxElectra::PostError(int32 ApiReturnValue, FString Message, int32 Code)
{
	LastError.Code = Code;
	LastError.SdkCode = ApiReturnValue;
	LastError.Message = MoveTemp(Message);
	return false;
}

void FVideoDecoderVPxElectra::Close()
{
	ResetToCleanStart();
	// Set the error state that all subsequent calls will fail.
	PostError(0, TEXT("Already closed"), ERRCODE_INTERNAL_ALREADY_CLOSED);
}

IElectraDecoder::ECSDCompatibility FVideoDecoderVPxElectra::IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions)
{
	if (!Decoders[EDecoderType::Color].Handle)
	{
		return IElectraDecoder::ECSDCompatibility::Compatible;
	}
	return IElectraDecoder::ECSDCompatibility::Drain;
}

bool FVideoDecoderVPxElectra::ResetToCleanStart()
{
	InternalDecoderDestroy();

	PendingDecoderInput.Empty();
	InDecoderInput.Empty();
	CurrentOutput.Reset();
	DecodeState = EDecodeState::Decoding;

	return !LastError.IsSet();
}

IElectraDecoder::EDecoderError FVideoDecoderVPxElectra::DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions)
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Can not feed new input until draining has finished.
	if (DecodeState == EDecodeState::Draining)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}

	// If we still have pending input we do not want anything new right now.
	if (!PendingDecoderInput.IsEmpty())
	{
		return IElectraDecoder::EDecoderError::NoBuffer;
	}

	// If there is pending output it is very likely that decoding this access unit would also generate output.
	// Since that would result in loss of the pending output we return now.
	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EDecoderError::NoBuffer;
	}

#if ELECTRA_MEDIAGPUBUFFER_DX12
	// If we will create a new resource pool or we have still buffers in an existing one, we can proceed, else we'd have no resources to output the data
	if (D3D12ResourcePool.IsValid() && !D3D12ResourcePool->BufferAvailable())
	{
		return IElectraDecoder::EDecoderError::NoBuffer;
	}
#endif

	// CSD only buffer is not handled at the moment.
	check((InInputAccessUnit.Flags & EElectraDecoderFlags::InitCSDOnly) == EElectraDecoderFlags::None);

	// Create decoder transform if necessary.
	if (!Decoders[EDecoderType::Color].Handle && !InternalDecoderCreate(InAdditionalOptions))
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Decode data. This immediately produces a new output frame.
	if (InInputAccessUnit.Data && InInputAccessUnit.DataSize)
	{
		switch(InitialCodecFormat.FourCC)
		{
			case Make4CC('v','p','0','8'):
			{
		        if (!PrepareSingleFrame(InInputAccessUnit, InAdditionalOptions))
		        {
			        return IElectraDecoder::EDecoderError::Error;
		        }
		        return DecodeNextPending();
			}
			case Make4CC('v','p','0','9'):
			{
		        // Decompose superframe into separate frames.
		        if (!DecomposeSuperFrame(InInputAccessUnit, InAdditionalOptions))
		        {
			        return IElectraDecoder::EDecoderError::Error;
		        }
		        return DecodeNextPending();
			}
			default:
			{
				PostError(0, TEXT("Unsupported format"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
				return IElectraDecoder::EDecoderError::Error;
			}
		}
	}
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EDecoderError FVideoDecoderVPxElectra::DecodeNextPending()
{
	TSharedPtr<FDecoderInput, ESPMode::NotThreadSafe> In;
	if (!PendingDecoderInput.Dequeue(In))
	{
		return IElectraDecoder::EDecoderError::None;
	}
	switch(InitialCodecFormat.FourCC)
	{
		case Make4CC('v','p','0','8'):
		{
			ElectraDecodersUtil::VPxVideo::FVP8UncompressedHeader Header;
			if (!ElectraDecodersUtil::VPxVideo::ParseVP8UncompressedHeader(Header, In->InputDataCopy, In->InputDataSize))
			{
				PostError(0, TEXT("Failed to validate VP8 bitstream header"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
				return IElectraDecoder::EDecoderError::Error;
			}
			break;
		}
		case Make4CC('v','p','0','9'):
		{
			ElectraDecodersUtil::VPxVideo::FVP9UncompressedHeader Header;
			if (!ElectraDecodersUtil::VPxVideo::ParseVP9UncompressedHeader(Header, In->InputDataCopy, In->InputDataSize))
			{
				PostError(0, TEXT("Failed to validate VP9 bitstream header"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
				return IElectraDecoder::EDecoderError::Error;
			}
			if (In->InputAlphaDataCopy && !ElectraDecodersUtil::VPxVideo::ParseVP9UncompressedHeader(Header, In->InputAlphaDataCopy, In->InputAlphaDataSize))
			{
				PostError(0, TEXT("Failed to validate VP9 alpha channel bitstream header"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
				return IElectraDecoder::EDecoderError::Error;
			}
			break;
		}
		default:
		{
			return IElectraDecoder::EDecoderError::Error;
		}
	}

	// Invoke decoder for the color channel
	vpx_codec_err_t Result = vpx_codec_decode(Decoders[EDecoderType::Color].Handle, reinterpret_cast<const uint8*>(In->InputDataCopy), (unsigned int) In->InputDataSize, In.Get(), 0);
	if (Result != VPX_CODEC_OK)
	{
		PostError(Result, TEXT("Failed to decode video decoder input"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
		return IElectraDecoder::EDecoderError::Error;
	}
	// Invoke decoder for alpha channel if there is one
	if (In->InputAlphaDataCopy)
	{
		check(Decoders[EDecoderType::Alpha].Handle);
		Result = vpx_codec_decode(Decoders[EDecoderType::Alpha].Handle, reinterpret_cast<const uint8*>(In->InputAlphaDataCopy), (unsigned int) In->InputAlphaDataSize, In.Get(), 0);
		if (Result != VPX_CODEC_OK)
		{
			PostError(Result, TEXT("Failed to decode video decoder alpha channel input"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
			return IElectraDecoder::EDecoderError::Error;
		}
	}

	// Add to the list of inputs passed to the decoder.
	InDecoderInput.Emplace(MoveTemp(In));
	InDecoderInput.StableSort([](const TSharedPtr<FDecoderInput, ESPMode::NotThreadSafe>& a, const TSharedPtr<FDecoderInput, ESPMode::NotThreadSafe>& b)
	{
		return a->AccessUnit.PTS < b->AccessUnit.PTS;
	});

	// Did we produce a new frame?
	Decoders[EDecoderType::Color].OutputIterator = nullptr;
	Decoders[EDecoderType::Alpha].OutputIterator = nullptr;
	EConvertResult ConvResult = ConvertDecoderOutput(vpx_codec_get_frame(Decoders[EDecoderType::Color].Handle, &Decoders[EDecoderType::Color].OutputIterator)
			, Decoders[EDecoderType::Alpha].Handle ? vpx_codec_get_frame(Decoders[EDecoderType::Alpha].Handle, &Decoders[EDecoderType::Alpha].OutputIterator) : nullptr);
	if (ConvResult == EConvertResult::Failure)
	{
		return IElectraDecoder::EDecoderError::Error;
	}
	return IElectraDecoder::EDecoderError::None;
}


bool FVideoDecoderVPxElectra::PrepareSingleFrame(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions)
{
	TSharedPtr<FDecoderInput, ESPMode::NotThreadSafe> In(new FDecoderInput);
	In->SuperFrameIndexPlusOne = 0;
	In->AdditionalOptions = InAdditionalOptions;
	In->AccessUnit = InInputAccessUnit;
	In->InputDataSize = InInputAccessUnit.DataSize;
	In->InputDataCopy = FMemory::Malloc(InInputAccessUnit.DataSize);
	FMemory::Memcpy(In->InputDataCopy, InInputAccessUnit.Data, InInputAccessUnit.DataSize);
	// Zero the input pointer and size in the copy. That data is not owned by us and it's best not to have any
	// values here that would lead us to think that we do.
	In->AccessUnit.Data = nullptr;
	In->AccessUnit.DataSize = 0;
	PendingDecoderInput.Enqueue(MoveTemp(In));
	return true;
}

bool FVideoDecoderVPxElectra::DecomposeSuperFrame(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions)
{
	TArray<uint8> AlphaFrameData = ElectraDecodersUtil::GetVariantValueUInt8Array(InAdditionalOptions, DynamicSidebandData::VPxAlpha.ToString());

	TArray<uint32> FrameSizes, AlphaFrameSizes;
	if (!ElectraDecodersUtil::VPxVideo::GetVP9SuperframeSizes(FrameSizes, InInputAccessUnit.Data, InInputAccessUnit.DataSize) ||
		(AlphaFrameData.Num() && !ElectraDecodersUtil::VPxVideo::GetVP9SuperframeSizes(AlphaFrameSizes, AlphaFrameData.GetData(), AlphaFrameData.Num())))
	{
		return false;
	}
	// If there is an alpha channel then it needs to have the same number of frames as the color channel.
	if (AlphaFrameData.Num() && FrameSizes.Num() != AlphaFrameSizes.Num())
	{
		return false;
	}

	// Create input frames.
	const uint8* Data = reinterpret_cast<const uint8*>(InInputAccessUnit.Data);
	const uint8* AlphaData = AlphaFrameData.Num() ? AlphaFrameData.GetData() : nullptr;
	for(int32 i=0; i<FrameSizes.Num(); ++i)
	{
		// Add to the list of inputs passed to the decoder.
		TSharedPtr<FDecoderInput, ESPMode::NotThreadSafe> In(new FDecoderInput);
		In->SuperFrameIndexPlusOne = FrameSizes.Num()>1 ? i+1 : 0;
		In->AdditionalOptions = InAdditionalOptions;
		In->AccessUnit = InInputAccessUnit;
		In->InputDataSize = FrameSizes[i];
		In->InputDataCopy = FMemory::Malloc(FrameSizes[i]);
		if (AlphaData)
		{
			In->InputAlphaDataSize = AlphaFrameSizes[i];
			In->InputAlphaDataCopy = FMemory::Malloc(AlphaFrameSizes[i]);
			FMemory::Memcpy(In->InputAlphaDataCopy, AlphaData, AlphaFrameSizes[i]);
			AlphaData += AlphaFrameSizes[i];
		}
		FMemory::Memcpy(In->InputDataCopy, Data, FrameSizes[i]);
		Data += FrameSizes[i];
		// Zero the input pointer and size in the copy. That data is not owned by us and it's best not to have any
		// values here that would lead us to think that we do.
		In->AccessUnit.Data = nullptr;
		In->AccessUnit.DataSize = 0;
		PendingDecoderInput.Enqueue(MoveTemp(In));
	}
	return true;
}

IElectraDecoder::EDecoderError FVideoDecoderVPxElectra::SendEndOfData()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Already draining?
	if (DecodeState == EDecodeState::Draining)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}

	// If there is a transform send an end-of-stream and drain message.
	if (Decoders[EDecoderType::Color].Handle)
	{
		DecodeState = EDecodeState::Draining;
	}
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EDecoderError FVideoDecoderVPxElectra::Flush()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}
	if (Decoders[EDecoderType::Color].Handle)
	{
		InternalDecoderDestroy();
		DecodeState = EDecodeState::Decoding;
		PendingDecoderInput.Empty();
		InDecoderInput.Empty();
		CurrentOutput.Reset();
	}
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EOutputStatus FVideoDecoderVPxElectra::HaveOutput()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EOutputStatus::Error;
	}
	// Have output?
	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EOutputStatus::Available;
	}

	// See if there is any additional pending output.
	if (Decoders[EDecoderType::Color].Handle && Decoders[EDecoderType::Color].OutputIterator)
	{
		EConvertResult ConvResult = ConvertDecoderOutput(vpx_codec_get_frame(Decoders[EDecoderType::Color].Handle, &Decoders[EDecoderType::Color].OutputIterator)
				, Decoders[EDecoderType::Alpha].Handle ? vpx_codec_get_frame(Decoders[EDecoderType::Alpha].Handle, &Decoders[EDecoderType::Alpha].OutputIterator) : nullptr);
		if (ConvResult == EConvertResult::Failure)
		{
			return IElectraDecoder::EOutputStatus::Error;
		}
		else if (ConvResult == EConvertResult::Success && CurrentOutput.IsValid())
		{
			return IElectraDecoder::EOutputStatus::Available;
		}
	}

	// Decode any pending input first, even when we are to drain the decoder.
	if (Decoders[EDecoderType::Color].Handle && !PendingDecoderInput.IsEmpty())
	{
		switch(DecodeNextPending())
		{
			case IElectraDecoder::EDecoderError::NoBuffer:
			{
				return IElectraDecoder::EOutputStatus::TryAgainLater;
			}
			case IElectraDecoder::EDecoderError::None:
			{
				if (CurrentOutput.IsValid())
				{
					return IElectraDecoder::EOutputStatus::Available;
				}
				break;
			}
			case IElectraDecoder::EDecoderError::Error:
			{
				return IElectraDecoder::EOutputStatus::Error;
			}
			default:
			{
				break;
			}
		}
	}
	if (DecodeState == EDecodeState::Draining && Decoders[EDecoderType::Color].Handle)
	{
		vpx_codec_err_t Result = vpx_codec_decode(Decoders[EDecoderType::Color].Handle, nullptr, 0, nullptr, 0);
		if (Decoders[EDecoderType::Alpha].Handle)
		{
			Result = vpx_codec_decode(Decoders[EDecoderType::Alpha].Handle, nullptr, 0, nullptr, 0);
		}

		EConvertResult ConvResult = ConvertDecoderOutput(vpx_codec_get_frame(Decoders[EDecoderType::Color].Handle, &Decoders[EDecoderType::Color].OutputIterator)
				, Decoders[EDecoderType::Alpha].Handle ? vpx_codec_get_frame(Decoders[EDecoderType::Alpha].Handle, &Decoders[EDecoderType::Alpha].OutputIterator) : nullptr);
		if (ConvResult == EConvertResult::Failure)
		{
			return IElectraDecoder::EOutputStatus::Error;
		}
		else if (ConvResult == EConvertResult::GotEOS)
		{
			DecodeState = EDecodeState::Decoding;
			PendingDecoderInput.Empty();
			InDecoderInput.Empty();
			return IElectraDecoder::EOutputStatus::EndOfData;
		}
		else
		{
			return CurrentOutput.IsValid() ? IElectraDecoder::EOutputStatus::Available : IElectraDecoder::EOutputStatus::EndOfData;
		}
	}
	return IElectraDecoder::EOutputStatus::NeedInput;
}

TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> FVideoDecoderVPxElectra::GetOutput()
{
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> Out = CurrentOutput;
	CurrentOutput.Reset();
	return Out;
}

bool FVideoDecoderVPxElectra::InternalDecoderCreate(const TMap<FString, FVariant>& InAdditionalOptions)
{
	FMemory::Memzero(Decoders[EDecoderType::Color].Context);
	FMemory::Memzero(Decoders[EDecoderType::Alpha].Context);
	vpx_codec_err_t Result = VPX_CODEC_INCAPABLE;
	switch(InitialCodecFormat.FourCC)
	{
		case Make4CC('v','p','0','8'):
		{
			vpx_codec_caps_t Capabilities = vpx_codec_get_caps(vpx_codec_vp8_dx());
			check((Capabilities & (VPX_CODEC_CAP_PUT_SLICE | VPX_CODEC_CAP_PUT_FRAME)) == 0);

			vpx_codec_flags_t Flags = 0;
			const int32 NumOfThreads = 1;
			const vpx_codec_dec_cfg_t CodecConfig = { NumOfThreads, 0, 0 };
			Result = vpx_codec_dec_init(&Decoders[EDecoderType::Color].Context, vpx_codec_vp8_dx(), &CodecConfig, Flags);
			break;
		}
		case Make4CC('v','p','0','9'):
		{
			vpx_codec_caps_t Capabilities = vpx_codec_get_caps(vpx_codec_vp9_dx());
			check((Capabilities & (VPX_CODEC_CAP_PUT_SLICE | VPX_CODEC_CAP_PUT_FRAME)) == 0);
			//   VPX_CODEC_CAP_HIGHBITDEPTH 0x4
			//   VPX_CODEC_CAP_EXTERNAL_FRAME_BUFFER 0x400000
			vpx_codec_flags_t Flags = 0;
			const int32 NumOfThreads = 1;
			const vpx_codec_dec_cfg_t CodecConfig = { NumOfThreads, 0, 0 };
			Result = vpx_codec_dec_init(&Decoders[EDecoderType::Color].Context, vpx_codec_vp9_dx(), &CodecConfig, Flags);

			// If there is an additional alpha channel we need a 2nd decoder
			if (InAdditionalOptions.Contains(DynamicSidebandData::VPxAlpha.ToString()) && Result == VPX_CODEC_OK)
			{
				Result = vpx_codec_dec_init(&Decoders[EDecoderType::Alpha].Context, vpx_codec_vp9_dx(), &CodecConfig, Flags);
				if (Result == VPX_CODEC_OK)
				{
					Decoders[EDecoderType::Alpha].Handle = &Decoders[EDecoderType::Alpha].Context;
					Decoders[EDecoderType::Alpha].OutputIterator = nullptr;
				}
			}
			break;
		}
	}
	if (Result != VPX_CODEC_OK)
	{
		return PostError(Result, TEXT("Failed to create decoder"), ERRCODE_INTERNAL_COULD_NOT_CREATE_DECODER);
	}
	Decoders[EDecoderType::Color].Handle = &Decoders[EDecoderType::Color].Context;
	Decoders[EDecoderType::Color].OutputIterator = nullptr;
	return true;
}

void FVideoDecoderVPxElectra::InternalDecoderDestroy()
{
	if (Decoders[EDecoderType::Alpha].Handle)
	{
		/*vpx_codec_err_t Result =*/ vpx_codec_destroy(Decoders[EDecoderType::Alpha].Handle);
		Decoders[EDecoderType::Alpha].Handle = nullptr;
		FMemory::Memzero(Decoders[EDecoderType::Alpha].Context);
		Decoders[EDecoderType::Alpha].OutputIterator = nullptr;
	}
	if (Decoders[EDecoderType::Color].Handle)
	{
		/*vpx_codec_err_t Result =*/ vpx_codec_destroy(Decoders[EDecoderType::Color].Handle);
		Decoders[EDecoderType::Color].Handle = nullptr;
		FMemory::Memzero(Decoders[EDecoderType::Color].Context);
		Decoders[EDecoderType::Color].OutputIterator = nullptr;
	}
}



FVideoDecoderVPxElectra::EConvertResult FVideoDecoderVPxElectra::ConvertDecoderOutput(vpx_image_t* InDecodedImage, vpx_image_t* InDecodedAlphaImage)
{
	if (!InDecodedImage)
	{
		check(InDecodedAlphaImage == nullptr);
		Decoders[EDecoderType::Color].OutputIterator = nullptr;
		Decoders[EDecoderType::Alpha].OutputIterator = nullptr;
		return EConvertResult::GotEOS;
	}

	SCOPE_CYCLE_COUNTER(STAT_ElectraDecoder_ConvertOutputVpx);

	// Find the input corresponding to this output.
	TSharedPtr<FDecoderInput, ESPMode::NotThreadSafe> In;
	for(int32 nInDec=0; nInDec<InDecoderInput.Num(); ++nInDec)
	{
		if (InDecoderInput[nInDec].Get() == reinterpret_cast<FDecoderInput*>(InDecodedImage->user_priv))
		{
			In = InDecoderInput[nInDec];
			InDecoderInput.RemoveAt(nInDec);
			break;
		}
	}
	if (!In.IsValid())
	{
		PostError(0, TEXT("There is no matching decoder input for the decoded output!"), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
		return EConvertResult::Failure;
	}
	// Remove outdated entries.
	InDecoderInput.RemoveAll([FramePTS=In->AccessUnit.PTS](const TSharedPtr<FDecoderInput, ESPMode::NotThreadSafe>& e)
	{
		return e->AccessUnit.PTS < FramePTS;
	});


	TSharedPtr<FVideoDecoderOutputVPxElectra, ESPMode::ThreadSafe> NewOutput = MakeShared<FVideoDecoderOutputVPxElectra>();
	NewOutput->PTS = In->AccessUnit.PTS;
	NewOutput->UserValue = In->AccessUnit.UserValue;

	NewOutput->Width = InDecodedImage->d_w;
	NewOutput->Height = InDecodedImage->d_h;
	NewOutput->NumBits = InDecodedImage->bit_depth;
	NewOutput->AspectW = 1;
	NewOutput->AspectH = 1;

	void* PlatformDevice = nullptr;
	int32 PlatformDeviceVersion = 0;
	bool bUseGPUBuffers = false;
#if ELECTRA_MEDIAGPUBUFFER_DX12
	FElectraDecodersPlatformResources::GetD3DDeviceAndVersion(&PlatformDevice, &PlatformDeviceVersion);
	bUseGPUBuffers = (PlatformDevice && PlatformDeviceVersion >= 12000);
#endif

	if (InDecodedImage->fmt == VPX_IMG_FMT_I420)
	{
		if (InDecodedAlphaImage == nullptr)
		{
			NewOutput->NumBuffers = 1;
			NewOutput->ColorBufferFormat = EPixelFormat::PF_NV12;
			NewOutput->ColorBufferEncoding = EElectraTextureSamplePixelEncoding::Native;
			NewOutput->DecodedWidth = NewOutput->Width;
			NewOutput->DecodedHeight = bUseGPUBuffers ? NewOutput->Height : (NewOutput->Height * 3 / 2);
			if (!ConvertDecodedImageToNV12orP010(NewOutput.Get(), InDecodedImage, PlatformDevice, PlatformDeviceVersion))
			{
				PostError(0, FString::Printf(TEXT("Failed to convert decoded image")), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
				return EConvertResult::Failure;
			}
		}
		else
		{
			NewOutput->NumBuffers = 1;
			NewOutput->ColorBufferFormat = EPixelFormat::PF_B8G8R8A8;
			NewOutput->ColorBufferEncoding = EElectraTextureSamplePixelEncoding::YCbCr_Alpha;
			NewOutput->DecodedWidth = NewOutput->Width;
			NewOutput->DecodedHeight = NewOutput->Height;
			if (!ConvertDecodedImageWithAlpha(NewOutput.Get(), InDecodedImage, InDecodedAlphaImage, PlatformDevice, PlatformDeviceVersion))
			{
				PostError(0, FString::Printf(TEXT("Failed to convert decoded image")), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
				return EConvertResult::Failure;
			}
		}
	}
	else if (InDecodedImage->fmt == VPX_IMG_FMT_I42016)
	{
		check(NewOutput->NumBits == 10);
		if (!ConvertDecodedImageToNV12orP010(NewOutput.Get(), InDecodedImage, PlatformDevice, PlatformDeviceVersion))
		{
			PostError(0, FString::Printf(TEXT("Failed to convert decoded image")), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
			return EConvertResult::Failure;
		}
		NewOutput->NumBuffers = 1;
		NewOutput->ColorBufferFormat = EPixelFormat::PF_P010;
		NewOutput->ColorBufferEncoding = EElectraTextureSamplePixelEncoding::Native;
		NewOutput->DecodedWidth = NewOutput->Width;
		NewOutput->DecodedHeight = bUseGPUBuffers ? NewOutput->Height : (NewOutput->Height * 3 / 2);

		// VPx decoders return the 10-bit output in the lower bits, but the output pipe expects it in the upper bits. Post scale to compensate!
		NewOutput->ExtraValues.Emplace(TEXT("pix_datascale"), FVariant(64.0));
	}
	else
	{
		PostError(0, FString::Printf(TEXT("Unsupported decoded image format (%d)"), InDecodedImage->fmt), ERRCODE_INTERNAL_FAILED_TO_CONVERT_OUTPUT_SAMPLE);
		return EConvertResult::Failure;
	}

	switch(InitialCodecFormat.FourCC)
	{
		case Make4CC('v','p','0','8'):
		{
			NewOutput->ExtraValues.Emplace(TEXT("codec"), FVariant(TEXT("vp8")));
			break;
		}
		case Make4CC('v','p','0','9'):
		{
			NewOutput->ExtraValues.Emplace(TEXT("codec"), FVariant(TEXT("vp9")));
			break;
		}
	}
	NewOutput->ExtraValues.Emplace(TEXT("platform"), FVariant(TEXT("generic")));

	CurrentOutput = MoveTemp(NewOutput);
	return EConvertResult::Success;
}

bool FVideoDecoderVPxElectra::ConvertDecodedImageToNV12orP010(FVideoDecoderOutputVPxElectra* InNewOutput, const vpx_image_t* InDecodedImage, void* PlatformDevice, int32 PlatformDeviceVersion) const
{
	const bool bIsNV12 = (InDecodedImage->fmt == VPX_IMG_FMT_I420);

	const int32 w = InDecodedImage->d_w;
	const int32 h = InDecodedImage->d_h;
	const int32 aw = Align(w, 2);
	const int32 ah = Align(h, 2);

	const uint8* SrcY = (const uint8*)InDecodedImage->planes[0];
	const uint8* SrcU = (const uint8*)InDecodedImage->planes[1];
	const uint8* SrcV = (const uint8*)InDecodedImage->planes[2];
	const int32 PitchY = InDecodedImage->stride[0];
	const int32 PitchU = InDecodedImage->stride[1];
	const int32 PitchV = InDecodedImage->stride[2];
	if (!SrcY || !SrcU || !SrcV)
	{
		return false;
	}

	uint8* DstY = nullptr;
	uint8* DstUV = nullptr;
	uint32 DstPitch = 0;

	TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe>& OutNV12Buffer = InNewOutput->ColorBuffer;
#if ELECTRA_MEDIAGPUBUFFER_DX12
	if (!PlatformDevice || PlatformDeviceVersion < 12000)
#endif
	{
		uint32 Pitch = bIsNV12 ? aw : (aw * 2);
		int32 AllocSize = Pitch * (ah * 3 / 2);

#if ELECTRA_MEDIAGPUBUFFER_DX12
		InNewOutput->GPUBuffer.Resource = nullptr;
		InNewOutput->GPUBuffer.Fence = nullptr;
#endif

		OutNV12Buffer = MakeShared<TArray64<uint8>, ESPMode::ThreadSafe>();
		OutNV12Buffer->AddUninitialized(AllocSize);
		DstY = OutNV12Buffer->GetData();
		DstUV = DstY + Pitch * ah;
		DstPitch = Pitch;
	}
#if ELECTRA_MEDIAGPUBUFFER_DX12
	else
	{
		TRefCountPtr D3D12Device(static_cast<ID3D12Device*>(PlatformDevice));
		HRESULT Result;
		FString ResultMsg;

		OutNV12Buffer.Reset();
		InNewOutput->GPUBuffer.Resource = nullptr;

		// Create the resource pool as needed...
		if (!D3D12ResourcePool)
		{
			D3D12ResourcePool = MakeShared<FElectraMediaDecoderOutputBufferPool_DX12, ESPMode::ThreadSafe>(D3D12Device, MaxOutputBuffers, MaxWidth, MaxHeight * 3 / 2, bIsNV12 ? 1 : 2);
		}

		// Request resource and fence...
		uint32 BufferPitch;
		if (D3D12ResourcePool->AllocateOutputDataAsBuffer(Result, ResultMsg, InNewOutput->GPUBuffer, BufferPitch))
		{
			// Correct pitch to reflect resource's setup
			InNewOutput->ColorPitch = BufferPitch;

			// Map the buffer so we can get a CPU address to the WC configured buffer
			Result = InNewOutput->GPUBuffer.Resource->Map(0, nullptr, (void**)&DstY);
			if (FAILED(Result))
			{
				return false;
			}
			DstUV = DstY + BufferPitch * ah;
			DstPitch = BufferPitch;
		}
		else
		{
			return false;
		}
	}
#endif // ELECTRA_MEDIAGPUBUFFER_DX12

	InNewOutput->ColorPitch = DstPitch;

	if (bIsNV12)
	{
		for (int32 y = 0; y < h; ++y)
		{
			FMemory::Memcpy(DstY, SrcY, w);
			SrcY += PitchY;
			DstY += DstPitch;
		}
		for (int32 v = 0; v < h / 2; ++v)
		{
			uint8* DstUVLine = DstUV;
			for (int32 u = 0; u < w / 2; ++u)
			{
				*DstUVLine++ = SrcU[u];
				*DstUVLine++ = SrcV[u];
			}
			SrcU += PitchU;
			SrcV += PitchV;
			DstUV += DstPitch;
		}
	}
	else
	{
		// note: data is delivered in the lower 10-bits, but expected in the upper
		// -> instead of processing the data here, we provide a "data scale" attribute to be applied on conversion from YUV to RGB
		for (int32 y = 0; y < h; ++y)
		{
			FMemory::Memcpy(DstY, SrcY, int64(w) << 1);
			SrcY += PitchY;
			DstY += DstPitch;
		}
		for (int32 v = 0; v < h / 2; ++v)
		{
			uint16* DstUVLine = (uint16*)DstUV;
			const uint16* SrcU16 = (const uint16*)SrcU;
			const uint16* SrcV16 = (const uint16*)SrcV;
			for (int32 u = 0; u < w / 2; ++u)
			{
				*DstUVLine++ = SrcU16[u];
				*DstUVLine++ = SrcV16[u];
			}
			SrcU += PitchU;
			SrcV += PitchV;
			DstUV += DstPitch;
		}
	}

#if ELECTRA_MEDIAGPUBUFFER_DX12
	if (InNewOutput->GPUBuffer.Resource.IsValid())
	{
		InNewOutput->GPUBuffer.Resource->Unmap(0, nullptr);
		// To be compatible with implementations that might do the copy into the resource async, we also signal a fence
		// (strictly speaking we would not need to as this is all 100% synchronous and done before the GPU ever attempts to read from the resource)
		InNewOutput->GPUBuffer.Fence->Signal(InNewOutput->GPUBuffer.FenceValue);
	}
#endif

	return true;
}

bool FVideoDecoderVPxElectra::ConvertDecodedImageWithAlpha(FVideoDecoderOutputVPxElectra* InNewOutput, const vpx_image_t* InDecodedImage, const vpx_image_t* InDecodedAlphaImage, void* PlatformDevice, int32 PlatformDeviceVersion) const
{
	// Can handle 8 bit 4:2:0 only at the moment.
	// Would the alpha channel be 16 bit as well if the color channel is 16 bit?
	if (InDecodedImage->fmt != VPX_IMG_FMT_I420)
	{
		return false;
	}

	const int32 w = InDecodedImage->d_w;
	const int32 h = InDecodedImage->d_h;
	const int32 aw = Align(w, 2);
	const int32 ah = Align(h, 2);

	const uint8* S0Y = (const uint8*)InDecodedImage->planes[0];
	const uint8* S0A = (const uint8*)InDecodedAlphaImage->planes[0];
	const uint8* SrcU = (const uint8*)InDecodedImage->planes[1];
	const uint8* SrcV = (const uint8*)InDecodedImage->planes[2];
	const int32 PitchY = InDecodedImage->stride[0];
	const int32 PitchA = InDecodedAlphaImage->stride[0];
	const int32 PitchU = InDecodedImage->stride[1];
	const int32 PitchV = InDecodedImage->stride[2];
	if (!S0Y || !SrcU || !SrcV || !S0A || PitchY != PitchA)
	{
		return false;
	}


	TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe>& OutPixelBuffer = InNewOutput->ColorBuffer;
	uint16* Dst0 = nullptr;
	uint16* Dst1 = nullptr;
	int32 DstPitch = 0;
	int32 LineSkipDst = 0;

#if ELECTRA_MEDIAGPUBUFFER_DX12
	if (!PlatformDevice || PlatformDeviceVersion < 12000)
#endif
	{
#if ELECTRA_MEDIAGPUBUFFER_DX12
		InNewOutput->GPUBuffer.Resource = nullptr;
		InNewOutput->GPUBuffer.Fence = nullptr;
#endif

		int32 AllocSize = aw * ah * sizeof(uint16) * 4;
		OutPixelBuffer = MakeShared<TArray64<uint8>, ESPMode::ThreadSafe>();
		OutPixelBuffer->AddUninitialized(AllocSize);
		Dst0 = (uint16*) OutPixelBuffer->GetData();
		InNewOutput->ColorPitch = aw * sizeof(uint16) * 4;
		DstPitch = aw * 4;
		Dst1 = Dst0 + DstPitch;
	}
#if ELECTRA_MEDIAGPUBUFFER_DX12
	else
	{
		TRefCountPtr D3D12Device(static_cast<ID3D12Device*>(PlatformDevice));
		HRESULT Result;
		FString ResultMsg;

		OutPixelBuffer.Reset();
		InNewOutput->GPUBuffer.Resource = nullptr;

		// Create the resource pool as needed...
		if (!D3D12ResourcePool)
		{
			const uint32 BytesPerPixel = 8;
			D3D12ResourcePool = MakeShared<FElectraMediaDecoderOutputBufferPool_DX12, ESPMode::ThreadSafe>(D3D12Device, MaxOutputBuffers, MaxWidth, MaxHeight, BytesPerPixel);
		}

		// Request resource and fence...
		uint32 BufferPitch;
		if (D3D12ResourcePool->AllocateOutputDataAsBuffer(Result, ResultMsg, InNewOutput->GPUBuffer, BufferPitch))
		{
			// Map the buffer so we can get a CPU address to the WC configured buffer
			Result = InNewOutput->GPUBuffer.Resource->Map(0, nullptr, (void**)&Dst0);
			if (FAILED(Result))
			{
				return false;
			}
			InNewOutput->ColorPitch = BufferPitch;
			DstPitch = BufferPitch / sizeof(uint16);
			LineSkipDst = ((DstPitch / 4) - aw) * 4;
			Dst1 = Dst0 + DstPitch;
		}
		else
		{
			return false;
		}
	}
#endif
	InNewOutput->NumBits = 16;
	InNewOutput->ColorBufferFormat = EPixelFormat::PF_A16B16G16R16;
	InNewOutput->ColorBufferEncoding = EElectraTextureSamplePixelEncoding::YCbCr_Alpha;

	const int32 LineSkipY = PitchY - w;
	const int32 LineSkipUV = PitchU - w/2;
	const uint8* S1Y = S0Y + PitchY;
	const uint8* S1A = S0A + PitchA;
	#define EXPAND16(v) (((uint16)v << 8) | (uint16)v)
	for(int32 y=0; y<h/2; ++y)
	{
		for(int32 x=0; x<w/2; ++x)
		{
			Dst0[0] = EXPAND16(S0A[0]);
			Dst0[1] = EXPAND16(S0Y[0]);
			Dst0[4] = EXPAND16(S0A[1]);
			Dst0[5] = EXPAND16(S0Y[1]);
			Dst1[0] = EXPAND16(S1A[0]);
			Dst1[1] = EXPAND16(S1Y[0]);
			Dst1[4] = EXPAND16(S1A[1]);
			Dst1[5] = EXPAND16(S1Y[1]);
			Dst0[2] = Dst0[6] = Dst1[2] = Dst1[6] = EXPAND16(*SrcU);
			Dst0[3] = Dst0[7] = Dst1[3] = Dst1[7] = EXPAND16(*SrcV);
			S0A += 2;
			S0Y += 2;
			S1A += 2;
			S1Y += 2;
			++SrcU;
			++SrcV;
			Dst0 += 8;
			Dst1 += 8;
		}
		S0Y += LineSkipY + PitchY;
		S1Y += LineSkipY + PitchY;
		S0A += LineSkipY + PitchA;
		S1A += LineSkipY + PitchA;
		SrcU += LineSkipUV;
		SrcV += LineSkipUV;
		Dst0 += LineSkipDst + DstPitch;
		Dst1 += LineSkipDst + DstPitch;
	}

#if ELECTRA_MEDIAGPUBUFFER_DX12
	if (InNewOutput->GPUBuffer.Resource.IsValid())
	{
		InNewOutput->GPUBuffer.Resource->Unmap(0, nullptr);
		// To be compatible with implementations that might do the copy into the resource async, we also signal a fence
		// (strictly speaking we would not need to as this is all 100% synchronous and done before the GPU ever attempts to read from the resource)
		InNewOutput->GPUBuffer.Fence->Signal(InNewOutput->GPUBuffer.FenceValue);
	}
#endif
	return true;
}


