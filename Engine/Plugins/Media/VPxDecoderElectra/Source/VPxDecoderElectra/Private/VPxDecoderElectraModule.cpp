// Copyright Epic Games, Inc. All Rights Reserved.

#include "VPxDecoderElectraModule.h"
#include "VPxDecoder/ElectraMediaVPxDecoder.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "VPxDecoderElectraModule"

DEFINE_LOG_CATEGORY(LogVPxElectraDecoder);

class FVPxElectraDecoderModule : public IModuleInterface
{
public:
	void StartupModule() override
	{
		FElectraMediaVPxDecoder::Startup();
	}

	void ShutdownModule() override
	{
		FElectraMediaVPxDecoder::Shutdown();
	}

	bool SupportsDynamicReloading() override
	{
		// Codec could still be in use
		return false;
	}
};

IMPLEMENT_MODULE(FVPxElectraDecoderModule, VPxDecoderElectra);

#undef LOCTEXT_NAMESPACE
