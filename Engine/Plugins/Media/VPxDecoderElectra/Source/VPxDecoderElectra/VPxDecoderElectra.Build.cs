// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class VPxDecoderElectra : ModuleRules
	{
		public VPxDecoderElectra(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"ElectraBase",
                    "ElectraSamples",
					"ElectraCodecFactory",
					"ElectraDecoders",
					"LibVpx"
                });

			if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
			{
				PublicDependencyModuleNames.Add("DirectX");
				AddEngineThirdPartyPrivateStaticDependencies(Target, "DX12");

				if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) && Target.WindowsPlatform.Architecture != UnrealArch.Arm64)
				{
					PublicAdditionalLibraries.AddRange(new string[] {
						Path.Combine(Target.WindowsPlatform.DirectXLibDir, "dxerr.lib"),
					});
				}
			}
		}
	}
}
